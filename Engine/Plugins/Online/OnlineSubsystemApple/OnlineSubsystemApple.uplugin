{
	"FileVersion" : 3,

	"FriendlyName" : "Online Subsystem Apple",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to Sign in with Apple platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,

	"SupportedTargetPlatforms": [
		"Mac",
		"IOS",
		"TVOS",
	],
    "Modules": [
        {
            "Name": "OnlineSubsystemApple",
            "Type": "Runtime",
            "LoadingPhase": "Default",
            "PlatformAllowList" :
			[
                "Mac",
                "IOS",
                "TVOS"
			]
        }
    ],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		}
	]

}