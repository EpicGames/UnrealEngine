// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

public class OnlineSubsystemApple : ModuleRules
{
	bool bSignInWithAppleSupported = false;

	public OnlineSubsystemApple(ReadOnlyTargetRules Target) : base(Target)
    {
		bRequiresPlatformSDK = true;
		
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			ConfigHierarchy PlatformGameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(Target.ProjectFile), UnrealTargetPlatform.IOS);
			PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableSignInWithAppleSupport", out bSignInWithAppleSupported);
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			// TODO: Mark.Fitt enable Mac support
		}

		PublicDefinitions.Add("ONLINESUBSYSTEMAPPLE_PACKAGE=1");
		PublicDefinitions.Add("ONLINESUBSYSTEMAPPLE_IDENTITY_ENABLE_SIWA=" + (bSignInWithAppleSupported ? "1" : "0"));

		PrivateDependencyModuleNames.AddRange(
		new string[] {
			"Core",
			"CoreUObject",
			"Engine",
			"OnlineSubsystem",
        });

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
		{
			if (bSignInWithAppleSupported)
			{
				PublicWeakFrameworks.Add("AuthenticationServices");
			}
		}
	}
}
