// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineExternalUIInterfaceApple.h"
#include "OnlineSubsystemApple.h"
#include "OnlineSubsystemAppleAttributes.h"
#include "Misc/ConfigCacheIni.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS || PLATFORM_TVOS
#include "IOS/IOSAppDelegate.h"
#endif

#import <Foundation/Foundation.h>

#if ONLINESUBSYSTEMAPPLE_IDENTITY_ENABLE_SIWA
	#define SIWA_SUPPORTED 1
#else
	#define SIWA_SUPPORTED 0
#endif

#if SIWA_SUPPORTED
#import <AuthenticationServices/AuthenticationServices.h>
#endif


DECLARE_MULTICAST_DELEGATE_FiveParams(FOnSignInComplete, NSString* /*user*/, NSData* /*authorizationCode*/, NSString* /*email*/, NSPersonNameComponents* /*fullName*/, NSError* /*error*/);
typedef FOnSignInComplete::FDelegate FOnSignInCompleteDelegate;


// Sign in With Apple interface to supply required deleagates for fulfilment of requests

typedef void (*SignInControllerResultCallback)(NSString* user, NSData* authorizationCode, NSString* email, NSPersonNameComponents* fullName, NSError* error);


#if SIWA_SUPPORTED
@interface SignInController : NSObject <ASAuthorizationControllerDelegate, ASAuthorizationControllerPresentationContextProviding>
#else
@interface SignInController : NSObject
#endif
{
	class SignInWithAppleBridge* signInWithAppleBridge;
}

- (instancetype)initWithBridge:(class SignInWithAppleBridge*)bridge;
@end


// Bridge SiWA requests and handle callbacks once completed

class SignInWithAppleBridge
{
public:
	/**
	 * Delegate called when a sign in attempt has completed successfully, or failed
	 *
	 * @param user - idenitfier for the logged in user, or null if failed
	 * @param authorizationCode - short-lived token for proof of authorization, or null if failed
	 * @param email - email address of the logged in user, or null if not in requested scope, user chose not to share or failed
	 * @param fullName - name components of the logged in user, or null if not in requested scope, user chose not to share or failed
	 * @param error - if the attempt failed, can be null if no attempt was made or successful
	 */
	DEFINE_ONLINE_DELEGATE_FIVE_PARAM(OnSignInComplete, NSString* /*user*/, NSData* /*authorizationCode*/, NSString* /*email*/, NSPersonNameComponents* /*fullName*/, NSError* /*error*/);

	SignInWithAppleBridge()
	{
		Controller = [[SignInController alloc] initWithBridge:this];
	}

	~SignInWithAppleBridge()
	{
		if (Controller != nullptr)
		{
			[Controller release];
			Controller = nullptr;
		}
	}

	SignInController* GetSignInController() const
	{
		return Controller;
	}

	void SignInResultCallback(NSString* user, NSData* authorizationCode, NSString* email, NSPersonNameComponents* fullName, NSError* error)
	{
		TriggerOnSignInCompleteDelegates(user, authorizationCode, email, fullName, error);
	}

private:
	SignInController* Controller;
};


@implementation SignInController

- (instancetype)initWithBridge:(class SignInWithAppleBridge*)bridge;
{
	self = [super init];
	if (self)
	{
		signInWithAppleBridge = bridge;
	}
	return self;
}

#if SIWA_SUPPORTED
- (ASPresentationAnchor)presentationAnchorForAuthorizationController:(ASAuthorizationController *)controller
{
#if PLATFORM_MAC
	// TODO: Mark.Fitt Support Mac window access
	return nullptr;
#elif PLATFORM_IOS || PLATFORM_TVOS
	if ([IOSAppDelegate GetDelegate].Window == nullptr)
	{
		NSLog(@"authorizationController: presentationAnchorForAuthorizationController: error window is NULL");
	}
	return [IOSAppDelegate GetDelegate].Window;
#endif
}

- (void)authorizationController:(ASAuthorizationController *)controller didCompleteWithAuthorization:(ASAuthorization *)authorization
{
	if ([authorization.credential isKindOfClass:[ASAuthorizationAppleIDCredential class]])
	{
		ASAuthorizationAppleIDCredential* AuthAppleID = (ASAuthorizationAppleIDCredential *)authorization.credential;
//DEBUG:>>
//Debug		NSLog(@"authorizationController: didCompleteWithAuthorization: identityToken %@", [AuthAppleID identityToken]);
//Debug		NSLog(@"authorizationController: didCompleteWithAuthorization: authorizationCode %@", [AuthAppleID authorizationCode]);
//Debug		NSLog(@"authorizationController: didCompleteWithAuthorization: state %@", [AuthAppleID state]);
//Debug		NSLog(@"authorizationController: didCompleteWithAuthorization: user %@", [AuthAppleID user]);

//Debug		NSArray *Scopes = [AuthAppleID authorizedScopes];
//Debug		NSLog(@"authorizationController: didCompleteWithAuthorization: scopes count %lu", (unsigned long)[Scopes count]);
//Debug		for (int i = 0; i < [Scopes count]; ++i) {
//Debug			NSLog(@"authorizationController: didCompleteWithAuthorization: scopes %@", Scopes[i]);
//Debug		}

//Debug		NSLog(@"authorizationController: didCompleteWithAuthorization: email %@", [AuthAppleID email]);
//Debug		NSLog(@"authorizationController: didCompleteWithAuthorization: fullName %@", [AuthAppleID fullName]);
//DEBUG:<<

		if (signInWithAppleBridge != nullptr)
		{
			signInWithAppleBridge->SignInResultCallback([AuthAppleID user], [AuthAppleID identityToken], [AuthAppleID email], [AuthAppleID fullName], nullptr);
		}
	}
	else
	{
		// Not the response for the requested auth type
		if (signInWithAppleBridge != nullptr)
		{
			signInWithAppleBridge->SignInResultCallback(nullptr, nullptr, nullptr, nullptr, nullptr);
		}
	}
}

- (void)authorizationController:(ASAuthorizationController *)controller didCompleteWithError:(NSError *)error
{
	if (signInWithAppleBridge != nullptr)
	{
		signInWithAppleBridge->SignInResultCallback(nullptr, nullptr, nullptr, nullptr, error);
	}
}
#endif

@end


// FOnlineExternalUIApple

FOnlineExternalUIApple::FOnlineExternalUIApple(FOnlineSubsystemApple* InSubsystem)
	: Subsystem(InSubsystem)
{
	check(InSubsystem != nullptr);

	SignInBridge = new SignInWithAppleBridge();

	// Setup permission scope fields
	GConfig->GetArray(TEXT("OnlineSubsystemApple.OnlineIdentityApple"), TEXT("ScopeFields"), ScopeFields, GEngineIni);

//DEBUG:>>
//Debug	ScopeFields.AddUnique(TEXT(APPLE_PERM_EMAIL));
//Debug	ScopeFields.AddUnique(TEXT(APPLE_PERM_FULLNAME));
//DEBUG:>>
}

FOnlineExternalUIApple::~FOnlineExternalUIApple()
{
	if (SignInBridge)
	{
		delete SignInBridge;
		SignInBridge = nullptr;
	}
}

bool FOnlineExternalUIApple::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
    bool bStarted = false;
    FString ErrorStr;
    
#if SIWA_SUPPORTED
    if (ControllerIndex >= 0 && ControllerIndex < MAX_LOCAL_PLAYERS)
    {
        FOnlineIdentityApplePtr IdentityInterface = StaticCastSharedPtr<FOnlineIdentityApple>(Subsystem->GetIdentityInterface());
        if (IdentityInterface.IsValid())
        {
            // Is the user already logged in?
            if (IdentityInterface->GetLoginStatus(ControllerIndex)!= ELoginStatus::NotLoggedIn)
            {
                Subsystem->ExecuteNextTick([IdentityInterface, ControllerIndex, Delegate]()
                                           {
                    Delegate.ExecuteIfBound(IdentityInterface->GetUniquePlayerId(ControllerIndex), ControllerIndex, FOnlineError::Success());
                });
                bStarted = true;
            }
            else
            {
                LoginUIClosedDelegate = Delegate;
                
                // Show the Sign in with Apple login UI
                OnSignInCompleteHandle = SignInBridge->AddOnSignInCompleteDelegate_Handle(FOnSignInCompleteDelegate::CreateRaw(this, &FOnlineExternalUIApple::OnSignInComplete, ControllerIndex));
                
                ASAuthorizationAppleIDProvider *Provider = [ASAuthorizationAppleIDProvider new];
                ASAuthorizationAppleIDRequest *Request = Provider.createRequest;
                
                if (ScopeFields.Num() > 0)
                {
                    // Define any requested scopes
                    // NOTE: We will only receieve scopes for the initial login only, once the app is authorised extended details are no longer available
                    NSMutableArray* Permissions = [NSMutableArray array];
                    for (int32 ScopeIdx = 0; ScopeIdx < ScopeFields.Num(); ScopeIdx++)
                    {
                        if (ScopeFields[ScopeIdx].Equals(TEXT(APPLE_PERM_EMAIL), ESearchCase::CaseSensitive))
                        {
                            [Permissions addObject: ASAuthorizationScopeEmail];
                        }
                        else if (ScopeFields[ScopeIdx].Equals(TEXT(APPLE_PERM_FULLNAME), ESearchCase::CaseSensitive))
                        {
                            [Permissions addObject: ASAuthorizationScopeFullName];
                        }
                        else
                        {
                            UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Login invalid scope. %s"), *ScopeFields[ScopeIdx]);
                        }
                    }
                    
                    Request.requestedScopes = Permissions;
                }
                
                ASAuthorizationController *Controller = [[ASAuthorizationController alloc] initWithAuthorizationRequests:@[Request]];
                
                SignInController* SignInController = SignInBridge->GetSignInController();
                Controller.delegate = SignInController;
                Controller.presentationContextProvider = SignInController;
                [Controller performRequests];
                
                bStarted = true;
            }
        }
        else
        {
            ErrorStr = TEXT("ShowLoginUI: Missing identity interface");
        }
    }
    else
    {
        ErrorStr = FString::Printf(TEXT("ShowLoginUI: Invalid controller index (%d)"), ControllerIndex);
    }
    
#else
    ErrorStr = TEXT("Sign in with Apple is not available");
#endif
    
    if (!bStarted)
    {
        UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("%s"), *ErrorStr);

		FOnlineError Error(false);
		Error.SetFromErrorCode(MoveTemp(ErrorStr));
		Subsystem->ExecuteNextTick([ControllerIndex, Delegate, Error = MoveTemp(Error)]()
		{
			Delegate.ExecuteIfBound(nullptr, ControllerIndex, Error);
		});
    }
	return bStarted;
}

bool FOnlineExternalUIApple::ShowFriendsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUIApple::ShowInviteUI(int32 LocalUserNum, FName SessionName)
{
	return false;
}

bool FOnlineExternalUIApple::ShowAchievementsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUIApple::ShowLeaderboardUI(const FString& LeaderboardName)
{
	return false;
}

bool FOnlineExternalUIApple::ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate)
{
	FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);

	return true;
}

bool FOnlineExternalUIApple::CloseWebURL()
{
	return false;
}

bool FOnlineExternalUIApple::ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIApple::ShowAccountUpgradeUI(const FUniqueNetId& UniqueId)
{
	return false;
}

bool FOnlineExternalUIApple::ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIApple::ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	return false;
}

void FOnlineExternalUIApple::OnSignInComplete(NSString* user, NSData* authorizationCode, NSString* email, NSPersonNameComponents* fullName, NSError* error, int32 LocalUserNum)
{
	SignInBridge->ClearOnSignInCompleteDelegate_Handle(OnSignInCompleteHandle);

	bool bWasSuccessful = (user != nullptr);
	FString ErrorStr;

	if (bWasSuccessful)
	{
		const FString PlayerId(user);
		NSString* authorizationCodeStr = [[NSString alloc] initWithData:authorizationCode encoding:NSUTF8StringEncoding];

		TSharedRef<FUserOnlineAccountApple> User = MakeShared<FUserOnlineAccountApple>(PlayerId, FString(authorizationCodeStr));

		if (fullName)
		{
			NSString* fullNameStr = [NSPersonNameComponentsFormatter localizedStringFromPersonNameComponents:fullName style:NSPersonNameComponentsFormatterStyleDefault options:(NSPersonNameComponentsFormatterOptions)0];

			if ([fullNameStr length] > 0)
			{
				User->RealName = FString(fullNameStr);

				if ([fullName.givenName length] > 0)
				{
					User->UserAttributes.Emplace(APPLE_USER_ATTR_FIRSTNAME, FString(fullName.givenName));
				}

				if ([fullName.familyName length] > 0)
				{
					User->UserAttributes.Emplace(APPLE_USER_ATTR_LASTNAME, FString(fullName.familyName));
				}
			}
		}

		if ([email length] > 0)
		{
			User->UserAttributes.Emplace(APPLE_USER_ATTR_EMAIL, FString(email));
		}

		FOnlineIdentityApplePtr IdentityInterface = StaticCastSharedPtr<FOnlineIdentityApple>(Subsystem->GetIdentityInterface());
		if (IdentityInterface.IsValid())
		{
			IdentityInterface->AddCachedAccount(LocalUserNum, User);
			Subsystem->ExecuteNextTick([this, User, LocalUserNum]()
			{
				LoginUIClosedDelegate.ExecuteIfBound(User->GetUserId(), LocalUserNum, FOnlineError::Success());
			});
		}
	}
	else
	{
		// Handle error codes for known error types
		switch (error.code)
		{
#if SIWA_SUPPORTED
			case ASAuthorizationErrorCanceled:
				// The user canceled the authorization attempt.
				ErrorStr = LOGIN_CANCELLED;
				break;

			case ASAuthorizationErrorFailed:
				// The authorization attempt failed.
				ErrorStr = LOGIN_ERROR_AUTH_FAILURE;
				break;

			case ASAuthorizationErrorInvalidResponse:
				// The authorization request received an invalid response.
			case ASAuthorizationErrorNotHandled:
				// The authorization request wasn’t handled.
			case ASAuthorizationErrorUnknown:
				// The authorization attempt failed for an unknown reason.
				// Typical failure code for an invalid application setup
#endif
			default:
				{
					NSString* errstr = [error localizedDescription];
					ErrorStr = FString::Printf(TEXT("Login failure %s"), *FString(errstr));
				}
				break;
		}

		FOnlineError Error(bWasSuccessful);
		Error.SetFromErrorCode(MoveTemp(ErrorStr));
		Subsystem->ExecuteNextTick([this, LocalUserNum, Error = MoveTemp(Error)]()
		{
			LoginUIClosedDelegate.ExecuteIfBound(nullptr, LocalUserNum, Error);
		});
	}
}

