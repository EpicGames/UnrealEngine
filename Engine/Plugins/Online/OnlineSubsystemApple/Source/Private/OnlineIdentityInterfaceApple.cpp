// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityInterfaceApple.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemApple.h"
#include "OnlineError.h"

#if ONLINESUBSYSTEMAPPLE_IDENTITY_ENABLE_SIWA
	#define SIWA_SUPPORTED 1
#else
	#define SIWA_SUPPORTED 0
#endif

#if SIWA_SUPPORTED
#import <AuthenticationServices/AuthenticationServices.h>
#endif


// FUserOnlineAccountApple

FUniqueNetIdRef FUserOnlineAccountApple::GetUserId() const
{
	return UserIdPtr;
}

FString FUserOnlineAccountApple::GetRealName() const
{
	return RealName;
}

FString FUserOnlineAccountApple::GetDisplayName(const FString& Platform) const
{
	return RealName;
}

FString FUserOnlineAccountApple::GetAccessToken() const
{
	return AuthToken;
}

bool FUserOnlineAccountApple::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	const FString* FoundAttr = UserAttributes.Find(AttrName);
	if (FoundAttr != nullptr)
	{
		OutAttrValue = *FoundAttr;
		return true;
	}
	return false;
}

bool FUserOnlineAccountApple::SetUserAttribute(const FString& AttrName, const FString& AttrValue)
{
	UserAttributes[AttrName] = AttrValue;
	return true;
}

bool FUserOnlineAccountApple::GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return false;
}

// FOnlineIdentityApple

FOnlineIdentityApple::FOnlineIdentityApple(FOnlineSubsystemApple* InSubsystem)
	: Subsystem(InSubsystem)
{
}

FOnlineIdentityApple::~FOnlineIdentityApple()
{
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentityApple::GetUserAccount(const FUniqueNetId& UserId) const
{
	TSharedPtr<FUserOnlineAccount> Result;

	const TSharedRef<FUserOnlineAccountApple>* FoundUserAccount = UserAccounts.Find(UserId.ToString());
	if (FoundUserAccount != nullptr)
	{
		Result = *FoundUserAccount;
	}

	return Result;
}

TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentityApple::GetAllUserAccounts() const
{
	TArray<TSharedPtr<FUserOnlineAccount> > Result;

	for (FUserOnlineAccountAppleMap::TConstIterator It(UserAccounts); It; ++It)
	{
		Result.Add(It.Value());
	}

	return Result;
}

bool FOnlineIdentityApple::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnlineIdentityApple::Login"));

#if SIWA_SUPPORTED
    ELoginStatus::Type LoginStatus = GetLoginStatus(LocalUserNum);
    
    if (LoginStatus != ELoginStatus::NotLoggedIn)
    {
        Subsystem->ExecuteNextTick([this, LocalUserNum]()
                                   {
            TriggerOnLoginCompleteDelegates(LocalUserNum, true, *GetUniquePlayerId(LocalUserNum), TEXT("Already logged in"));
        });
        return false;
    }
    
    ensure(LoginStatus == ELoginStatus::NotLoggedIn);
    if (!AccountCredentials.Id.IsEmpty())
    {
        FString PlayerId = AccountCredentials.Id;
        
        FTCHARToUTF8 UserIdStr(*PlayerId);
        NSString *UserId = [NSString stringWithUTF8String:UserIdStr.Get()];
        
        // Attempt to login with existing credentials, we only validate the Id, no full account details are created or need storing
        ASAuthorizationAppleIDProvider *Provider = [ASAuthorizationAppleIDProvider new];
        [Provider getCredentialStateForUserID:UserId completion:^(ASAuthorizationAppleIDProviderCredentialState credentialState, NSError *error)
         {
            bool bWasSuccessful = false;
            FString ErrorStr;
            
            switch (credentialState) {
                case ASAuthorizationAppleIDProviderCredentialAuthorized:
                {
                    // We receive no additional information or updated token, only that the supplied Id is still valid, or not on this device/application
                    TSharedRef<FUserOnlineAccountApple> User = MakeShared<FUserOnlineAccountApple>(PlayerId, FString());
                    
                    // update/add cached entry for user
                    UserAccounts.Add(User->GetUserId()->ToString(), User);
                    // keep track of user ids for local users
                    UserIds.Add(LocalUserNum, User->GetUserId());
                    
                    bWasSuccessful = true;
                    
                    UE_LOG_ONLINE_IDENTITY(Display, TEXT("Apple login with credentials was successful"));
                }
                    break;
                    
                case ASAuthorizationAppleIDProviderCredentialRevoked:
                    ErrorStr = TEXT("Credentials have been revoked");
                    break;
                    
                case ASAuthorizationAppleIDProviderCredentialNotFound:
                default:
                    ErrorStr = TEXT("Credentials not found");
                    break;
            }
            
            if (error)
            {
                NSString* errstr = [error localizedDescription];
                ErrorStr = FString::Printf(TEXT("Login failure %s"), *FString(errstr));
            }
            
            OnLoginAttemptComplete(LocalUserNum, ErrorStr);
        }];
        
        return true;
    }
    else
    {
        IOnlineExternalUIPtr OnlineExternalUI = Subsystem->GetExternalUIInterface();
        if (OnlineExternalUI.IsValid())
        {
            FOnLoginUIClosedDelegate CompletionDelegate = FOnLoginUIClosedDelegate::CreateRaw(this, &FOnlineIdentityApple::OnExternalUILoginComplete);
            OnlineExternalUI->ShowLoginUI(LocalUserNum, true, false, CompletionDelegate);
            return true;
        }
        else
        {
            const FString ErrorStr = TEXT("External interface missing");
            OnLoginAttemptComplete(LocalUserNum, ErrorStr);
            return false;
        }
    }
#endif

	Subsystem->ExecuteNextTick([this, LocalUserNum]()
	{
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdApple::EmptyId(), TEXT("Sign in with Apple is not available"));
	});
	return false;
}


// Store account details from an external UI login
void FOnlineIdentityApple::AddCachedAccount(int32 LocalUserNum, TSharedRef<FUserOnlineAccountApple> User)
{
	// update/add cached entry for user
	UserAccounts.Add(User->GetUserId()->ToString(), User);
	// keep track of user ids for local users
	UserIds.Add(LocalUserNum, User->GetUserId());
}


void FOnlineIdentityApple::OnExternalUILoginComplete(FUniqueNetIdPtr UniqueId, const int ControllerIndex, const FOnlineError& Error)
{
	const FString& ErrorStr = Error.GetErrorCode();
	OnLoginAttemptComplete(ControllerIndex, ErrorStr);
}

void FOnlineIdentityApple::OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr)
{
	const FString ErrorStrCopy(ErrorStr);
	if (GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Apple login was successful"));
		FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
		check(UserId.IsValid());

		Subsystem->ExecuteNextTick([this, UserId, LocalUserNum, ErrorStrCopy]()
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *UserId, ErrorStrCopy);
			TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, *UserId);
		});
	}
	else
	{
		Subsystem->ExecuteNextTick([this, LocalUserNum, ErrorStrCopy]()
	    {
			TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdApple::EmptyId(), ErrorStrCopy);
		});
	}
}

bool FOnlineIdentityApple::Logout(int32 LocalUserNum)
{
	bool bTriggeredLogout = false;

	ELoginStatus::Type LoginStatus = GetLoginStatus(LocalUserNum);
	if (LoginStatus == ELoginStatus::LoggedIn)
	{
		UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnlineIdentityApple::Logout complete"));
		FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
		if (UserId.IsValid())
		{
			// remove cached user account
			UserAccounts.Remove(UserId->ToString());
		}
		else
		{
			UserId = FUniqueNetIdApple::EmptyId();
		}
		// remove cached user id
		UserIds.Remove(LocalUserNum);

		Subsystem->ExecuteNextTick([this, UserId, LocalUserNum]()
		{
			TriggerOnLogoutCompleteDelegates(LocalUserNum, true);
			TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *UserId);
		});

		bTriggeredLogout = true;
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."), LocalUserNum);
	}

	if (!bTriggeredLogout)
	{
		UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnlineIdentityApple::Logout didn't trigger logout"));
		Subsystem->ExecuteNextTick([this, LocalUserNum]()
	    {
			TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
		});
	}

	return bTriggeredLogout;
}

bool FOnlineIdentityApple::AutoLogin(int32 LocalUserNum)
{
	return Login(LocalUserNum, FOnlineAccountCredentials());
}

ELoginStatus::Type FOnlineIdentityApple::GetLoginStatus(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetLoginStatus(*UserId);
	}
	return ELoginStatus::NotLoggedIn;
}

ELoginStatus::Type FOnlineIdentityApple::GetLoginStatus(const FUniqueNetId& UserId) const
{
	TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(UserId);
	if (UserAccount.IsValid() && UserAccount->GetUserId()->IsValid())
	{
		return ELoginStatus::LoggedIn;
	}
	return ELoginStatus::NotLoggedIn;
}

FUniqueNetIdPtr FOnlineIdentityApple::GetUniquePlayerId(int32 LocalUserNum) const
{
	const FUniqueNetIdPtr* FoundId = UserIds.Find(LocalUserNum);

	if (FoundId != nullptr)
	{
		return *FoundId;
	}

	return nullptr;
}

FUniqueNetIdPtr FOnlineIdentityApple::CreateUniquePlayerId(uint8* Bytes, int32 Size)
{
	if (Bytes && Size == sizeof(uint64))
	{
		int32 StrLen = FCString::Strlen((TCHAR*)Bytes);
		if (StrLen > 0)
		{
			FString StrId((TCHAR*)Bytes);
			return FUniqueNetIdApple::Create(StrId);
		}
	}

	return nullptr;
}

FUniqueNetIdPtr FOnlineIdentityApple::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdApple::Create(Str);
}

FString FOnlineIdentityApple::GetPlayerNickname(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetPlayerNickname(*UserId);
	}
	return TEXT("");
}

FString FOnlineIdentityApple::GetPlayerNickname(const FUniqueNetId& UserId) const
{
	const TSharedRef<FUserOnlineAccountApple>* FoundUserAccount = UserAccounts.Find(UserId.ToString());
	if (FoundUserAccount != nullptr)
	{
		const TSharedRef<FUserOnlineAccountApple>& UserAccount = *FoundUserAccount;
		return UserAccount->GetDisplayName();
	}
	return TEXT("");
}

FString FOnlineIdentityApple::GetAuthToken(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(*UserId);
		if (UserAccount.IsValid())
		{
			return UserAccount->GetAccessToken();
		}
	}
	return FString();
}

void FOnlineIdentityApple::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityApple::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	Subsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}

void FOnlineIdentityApple::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	Subsystem->ExecuteNextTick([UserIdRef, Privilege, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, Privilege, (uint32)EPrivilegeResults::NoFailures);
	});
}

FPlatformUserId FOnlineIdentityApple::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& InUniqueNetId) const
{
	for (int i = 0; i < MAX_LOCAL_PLAYERS; ++i)
	{
		FUniqueNetIdPtr CurrentUniqueId = GetUniquePlayerId(i);
		if (CurrentUniqueId.IsValid() && (*CurrentUniqueId == InUniqueNetId))
		{
			return GetPlatformUserIdFromLocalUserNum(i);
		}
	}

	return PLATFORMUSERID_NONE;
}

FString FOnlineIdentityApple::GetAuthType() const
{
	return TEXT("apple");
}

