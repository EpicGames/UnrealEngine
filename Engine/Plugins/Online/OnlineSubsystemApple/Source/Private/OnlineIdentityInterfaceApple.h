// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemAppleTypes.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystemApplePackage.h"

class FUserOnlineAccountApple;

/** Apple sign in scopes */
#define APPLE_PERM_EMAIL "email"
#define APPLE_PERM_FULLNAME "fullName"

/** Mapping from user id to the internal online account info (only one per user) */
typedef TMap<FString, TSharedRef<FUserOnlineAccountApple> > FUserOnlineAccountAppleMap;

/**
 * Info associated with an user account generated by this online service
 */
class FUserOnlineAccountApple :
	public FUserOnlineAccount
{
public:

	// FOnlineUser

	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;
	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) override;

	// FUserOnlineAccount

	virtual FString GetAccessToken() const override;
	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FUserOnlineAccountApple

	explicit FUserOnlineAccountApple(const FString& InUserId = FString(), const FString& InAuthToken = FString())
		: UserIdPtr(FUniqueNetIdApple::Create(InUserId))
		, AuthToken(InAuthToken)
	{
	}

	virtual ~FUserOnlineAccountApple()
	{
	}

protected:
	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserIdPtr;
	/** Real name */
	FString RealName;
	/** Token which is provided to user once authenticated by the online service */
	FString AuthToken;

	/** Additional key/value pair data related to user attribution */
	TMap<FString, FString> UserAttributes;

private:

	/** Allow Apple identity to fill in our private members from it's callbacks */
	friend class FOnlineExternalUIApple;
};

/**
* Apple service implementation of the online identity interface
*/

class FOnlineIdentityApple
	: public IOnlineIdentity
{
private:
	FOnlineSubsystemApple* Subsystem;

PACKAGE_SCOPE:

	/**
	 * Default Constructor
	 */
	FOnlineIdentityApple(FOnlineSubsystemApple* InSubsystem);

public:
	//~ Begin IOnlineIdentity Interface
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI=EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual FString GetAuthType() const override;
	//~ End IOnlineIdentity Interface

public:
	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityApple();

	void AddCachedAccount(int32 LocalUserNum, TSharedRef<FUserOnlineAccountApple> User);

private:
	void OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr);

	/**
	 * Delegate fired when the call to ShowLoginUI completes
	 */
	void OnExternalUILoginComplete(FUniqueNetIdPtr UniqueId, const int ControllerIndex, const FOnlineError& Error);

	/** Users that have been registered/authenticated */
	FUserOnlineAccountAppleMap UserAccounts;
	/** Ids mapped to locally registered users */
	TMap<int32, FUniqueNetIdPtr > UserIds;
};


typedef TSharedPtr<FOnlineIdentityApple, ESPMode::ThreadSafe> FOnlineIdentityApplePtr;

