// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemApple.h"

FOnlineSubsystemApple::FOnlineSubsystemApple(FName InInstanceName)
	: FOnlineSubsystemImpl(APPLE_SUBSYSTEM, InInstanceName)
{
}

IOnlineSessionPtr FOnlineSubsystemApple::GetSessionInterface() const
{
	return nullptr;
}

IOnlineFriendsPtr FOnlineSubsystemApple::GetFriendsInterface() const
{
	return nullptr;
}

IOnlinePartyPtr FOnlineSubsystemApple::GetPartyInterface() const
{
	return nullptr;
}

IOnlineGroupsPtr FOnlineSubsystemApple::GetGroupsInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemApple::GetSharedCloudInterface() const
{
	return nullptr;
}

IOnlineUserCloudPtr FOnlineSubsystemApple::GetUserCloudInterface() const
{
	return nullptr;
}

IOnlineLeaderboardsPtr FOnlineSubsystemApple::GetLeaderboardsInterface() const
{
	return nullptr;
}

IOnlineVoicePtr FOnlineSubsystemApple::GetVoiceInterface() const
{
	return nullptr;
}

IOnlineExternalUIPtr FOnlineSubsystemApple::GetExternalUIInterface() const
{
	return ExternalUIInterface;
}

IOnlineTimePtr FOnlineSubsystemApple::GetTimeInterface() const
{
	return nullptr;
}

IOnlineIdentityPtr FOnlineSubsystemApple::GetIdentityInterface() const
{
	return IdentityInterface;
}

IOnlineTitleFilePtr FOnlineSubsystemApple::GetTitleFileInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemApple::GetEntitlementsInterface() const
{
	return nullptr;
}

IOnlineStoreV2Ptr FOnlineSubsystemApple::GetStoreV2Interface() const
{
	return nullptr;
}

IOnlinePurchasePtr FOnlineSubsystemApple::GetPurchaseInterface() const
{
	return nullptr;
}

IOnlineEventsPtr FOnlineSubsystemApple::GetEventsInterface() const
{
	return nullptr;
}

IOnlineAchievementsPtr FOnlineSubsystemApple::GetAchievementsInterface() const
{
	return nullptr;
}

IOnlineSharingPtr FOnlineSubsystemApple::GetSharingInterface() const
{
	return nullptr;
}

IOnlineUserPtr FOnlineSubsystemApple::GetUserInterface() const
{
	return nullptr;
}

IOnlineMessagePtr FOnlineSubsystemApple::GetMessageInterface() const
{
	return nullptr;
}

IOnlinePresencePtr FOnlineSubsystemApple::GetPresenceInterface() const
{
	return nullptr;
}

IOnlineChatPtr FOnlineSubsystemApple::GetChatInterface() const
{
	return nullptr;
}

IOnlineStatsPtr FOnlineSubsystemApple::GetStatsInterface() const
{
	return nullptr;
}

IOnlineTurnBasedPtr FOnlineSubsystemApple::GetTurnBasedInterface() const
{
	return nullptr;
}

IOnlineTournamentPtr FOnlineSubsystemApple::GetTournamentInterface() const
{
	return nullptr;
}

bool FOnlineSubsystemApple::Init()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemApple::Init()"));

	IdentityInterface = MakeShareable(new FOnlineIdentityApple(this));
	ExternalUIInterface = MakeShareable(new FOnlineExternalUIApple(this));

	return true;
}

bool FOnlineSubsystemApple::Tick(float DeltaTime)
{
	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}
	return true;
}

FText FOnlineSubsystemApple::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemApple", "OnlineServiceName", "Game Center");
}

bool FOnlineSubsystemApple::Shutdown()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemApple::Shutdown()"));

#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		UE_LOG_ONLINE(Display, TEXT(#Interface));\
		ensure(Interface.IsUnique()); \
		Interface = nullptr; \
	}

	DESTRUCT_INTERFACE(ExternalUIInterface);
	DESTRUCT_INTERFACE(IdentityInterface);

#undef DESTRUCT_INTERFACE

	return true;
}

FString FOnlineSubsystemApple::GetAppId() const
{
	return TEXT("");
}

bool FOnlineSubsystemApple::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}
	return false;
}

