// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemAppleModule.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemApple.h"

IMPLEMENT_MODULE(FOnlineSubsystemAppleModule, OnlineSubsystemApple);

/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactoryApple : public IOnlineFactory
{

private:
	/** Single instantiation of the Apple interface */
	static FOnlineSubsystemApplePtr AppleSingleton;

	virtual void DestroySubsystem()
	{
		if (AppleSingleton.IsValid())
		{
			AppleSingleton->Shutdown();
			AppleSingleton = nullptr;
		}
	}

public:
	FOnlineFactoryApple() {}

	virtual ~FOnlineFactoryApple()
	{
		DestroySubsystem();
	}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		if (!AppleSingleton.IsValid())
		{
			AppleSingleton = MakeShared<FOnlineSubsystemApple, ESPMode::ThreadSafe>(InstanceName);
			if (AppleSingleton->IsEnabled())
			{
				if(!AppleSingleton->Init())
				{
					UE_LOG_ONLINE(Warning, TEXT("FOnlineSubsystemAppleModule failed to initialize!"));
					DestroySubsystem();
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("FOnlineSubsystemAppleModule was disabled"));
				DestroySubsystem();
			}

			return AppleSingleton;
		}

		UE_LOG_ONLINE(Warning, TEXT("Can't create more than one instance of Apple online subsystem!"));
		return nullptr;
	}
};

FOnlineSubsystemApplePtr FOnlineFactoryApple::AppleSingleton = nullptr;

void FOnlineSubsystemAppleModule::StartupModule()
{
	SCOPED_BOOT_TIMING("FOnlineSubsystemAppleModule::StartupModule");

	UE_LOG_ONLINE(Display, TEXT("FOnlineSubsystemAppleModule::StartupModule()"));

	AppleFactory = new FOnlineFactoryApple();

	// Create and register our singleton factory with the main online subsystem for easy access
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(APPLE_SUBSYSTEM, AppleFactory);
}

void FOnlineSubsystemAppleModule::ShutdownModule()
{
	UE_LOG_ONLINE(Display, TEXT("FOnlineSubsystemAppleModule::ShutdownModule()"));

	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.UnregisterPlatformService(APPLE_SUBSYSTEM);

	delete AppleFactory;
	AppleFactory = nullptr;
}

