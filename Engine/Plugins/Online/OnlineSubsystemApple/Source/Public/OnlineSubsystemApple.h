// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"
#include "OnlineSubsystemImpl.h"
#include "OnlineSubsystemApplePackage.h"

#include "OnlineIdentityInterfaceApple.h"
#include "OnlineExternalUIInterfaceApple.h"

/**
 *	OnlineSubsystemApple - Implementation of the online subsystem for Apple services
 */
class ONLINESUBSYSTEMAPPLE_API FOnlineSubsystemApple
	: public FOnlineSubsystemImpl
{

public:
	virtual ~FOnlineSubsystemApple() = default;

	//~ Begin IOnlineSubsystem Interface
	virtual IOnlineSessionPtr GetSessionInterface() const override;
	virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	virtual IOnlinePartyPtr GetPartyInterface() const override;
	virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	virtual IOnlineVoicePtr GetVoiceInterface() const  override;
	virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;
	virtual IOnlineTimePtr GetTimeInterface() const override;
	virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override;
	virtual IOnlinePurchasePtr GetPurchaseInterface() const override;
	virtual IOnlineEventsPtr GetEventsInterface() const override;
	virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	virtual IOnlineSharingPtr GetSharingInterface() const override;
	virtual IOnlineUserPtr GetUserInterface() const override;
	virtual IOnlineMessagePtr GetMessageInterface() const override;
	virtual IOnlinePresencePtr GetPresenceInterface() const override;
	virtual IOnlineChatPtr GetChatInterface() const override;
	virtual IOnlineStatsPtr GetStatsInterface() const override;
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override;
	virtual IOnlineTournamentPtr GetTournamentInterface() const override;
	virtual bool Init() override;
	virtual bool Shutdown() override;
	virtual FString GetAppId() const override;
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	virtual bool Tick(float DeltaTime) override;
	virtual FText GetOnlineServiceName() const override;
	//~ End IOnlineSubsystem Interface

PACKAGE_SCOPE:
	/** Only the factory makes instances */
	FOnlineSubsystemApple() = delete;
	explicit FOnlineSubsystemApple(FName InInstanceName);

private:
	/** Interface to the Identity information */
	FOnlineIdentityApplePtr IdentityInterface;

	/** Interface to the external UI services */
	FOnlineExternalUIApplePtr ExternalUIInterface;
};

typedef TSharedPtr<FOnlineSubsystemApple, ESPMode::ThreadSafe> FOnlineSubsystemApplePtr;

