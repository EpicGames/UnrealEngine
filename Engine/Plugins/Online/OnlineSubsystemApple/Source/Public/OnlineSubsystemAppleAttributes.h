// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 *	FUserOnlineAccount User Attributes for additional requested scopes
 */

#define APPLE_USER_ATTR_EMAIL TEXT("email")
#define APPLE_USER_ATTR_FIRSTNAME TEXT("firstName")
#define APPLE_USER_ATTR_LASTNAME TEXT("lastName")

