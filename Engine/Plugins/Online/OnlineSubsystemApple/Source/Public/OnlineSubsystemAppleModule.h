// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Online subsystem module class (Sign in with Apple Implementation)
 * Code related to the handling of the Sign in with Apple module.
 */
class FOnlineSubsystemAppleModule : public IModuleInterface
{
private:

	/** Class responsible for creating instance(s) of the subsystem */
	class FOnlineFactoryApple* AppleFactory;

public:

	FOnlineSubsystemAppleModule() :
		AppleFactory(nullptr)
	{}

	virtual ~FOnlineSubsystemAppleModule() {}

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
	//~ End IModuleInterface Interface
};

